// Copyright Epic Games, Inc. All Rights Reserved.

#include "VoiceChat.h"

const TCHAR* LexToString(const EVoiceChatChannelType Value)
{
	switch (Value)
	{
		case EVoiceChatChannelType::NonPositional: return TEXT("NonPositional");
		case EVoiceChatChannelType::Positional: return TEXT("Positional");
		case EVoiceChatChannelType::Echo: return TEXT("Echo");
		default: checkNoEntry(); return TEXT("Invalid");
	}
}

const TCHAR* LexToString(const EVoiceChatAttenuationModel Value)
{
	switch (Value)
	{
		case EVoiceChatAttenuationModel::None: return TEXT("None");
		case EVoiceChatAttenuationModel::InverseByDistance: return TEXT("InverseByDistance");
		case EVoiceChatAttenuationModel::LinearByDistance: return TEXT("LinearByDistance");
		case EVoiceChatAttenuationModel::ExponentialByDistance: return TEXT("ExponentialByDistance");
		default: checkNoEntry(); return TEXT("Invalid");
	}
}

const TCHAR* LexToString(const EVoiceChatTransmitMode Value)
{
	switch (Value)
	{
		case EVoiceChatTransmitMode::None: return TEXT("None");
		case EVoiceChatTransmitMode::All: return TEXT("All");
		case EVoiceChatTransmitMode::SpecificChannels: return TEXT("SpecificChannels");
		default: checkNoEntry(); return TEXT("Invalid");
	}
}