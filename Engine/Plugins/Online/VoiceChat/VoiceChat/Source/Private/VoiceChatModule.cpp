// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class FVoiceChatModule : public IModuleInterface
{
};

IMPLEMENT_MODULE(FVoiceChatModule, VoiceChat)