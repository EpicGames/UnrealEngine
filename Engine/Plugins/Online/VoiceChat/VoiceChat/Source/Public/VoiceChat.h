// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Delegates/Delegate.h"
#include "Features/IModularFeature.h"
#include "Features/IModularFeatures.h"
#include "VoiceChatResult.h"

enum class EVoiceChatChannelType
{
	/** Non positional/2d audio channel */
	NonPositional,
	/** Positional/3d audio channel */
	Positional,
	/** Echo channel. Will only ever have one player and will echo anything you say */
	Echo
};

VOICECHAT_API const TCHAR* LexToString(const EVoiceChatChannelType Value);

enum class EVoiceChatAttenuationModel
{
	/** No attenuation is applied. The audio will drop to 0 at MaxDistance */
	None,
	/** The attenuation increases in inverse proportion to the distance. The Rolloff is the inverse of the slope of the attenuation curve. */
	InverseByDistance,
	/** The attenuation increases in linear proportion to the distance. The Rolloff is the negative slope of the attenuation curve. */
	LinearByDistance,
	/** The attenuation increases in inverse proportion to the distance raised to the power of the Rolloff. */
	ExponentialByDistance
};

VOICECHAT_API const TCHAR* LexToString(const EVoiceChatAttenuationModel Value);

struct FVoiceChatChannel3dProperties
{
	/** The model used to determine how loud audio is at different distances */
	EVoiceChatAttenuationModel AttenuationModel;
	/** The distance at which the sound will start to attenuate */
	float MinDistance;
	/** The distance at which sound will no longer be audible */
	float MaxDistance;
	/** How fast the sound attenuates with distance */
	float Rolloff;
};

enum class EVoiceChatTransmitMode
{
	/** Transmit to none of the channels you are in */
	None,
	/** Transmit to all of the channels you are in */
	All,
	/** Transmit to specific channel(s) */
	SpecificChannels,
};

VOICECHAT_API const TCHAR* LexToString(const EVoiceChatTransmitMode Value);

struct FVoiceChatCallStats
{
	/** Call length in seconds */
	double CallLength;
	/** Minimum measured network latency in seconds. Zero if no measurements made */
	double LatencyMinMeasuredSeconds;
	/** Maximum measured network latency in seconds. Zero if no measurements made */
	double LatencyMaxMeasuredSeconds;
	/** Average measured network latency in seconds. Zero if no measurements made */
	double LatencyAverageMeasuredSeconds;
	/** Total number of packets, both received and lost */
	int PacketsNumTotal;
	/** Total number of lost packets */
	int PacketsNumLost;
};

struct FVoiceChatDeviceInfo
{
	/** The display name for the device */
	FString DisplayName;
	/** The unique id for the device */
	FString Id;

	inline bool operator==(const FVoiceChatDeviceInfo& Other) const { return Id == Other.Id; }
	inline bool operator!=(const FVoiceChatDeviceInfo& Other) const { return !operator==(Other); }

	FString ToDebugString() const { return FString::Printf(TEXT("DisplayName=[%s] Id=[%s]"), *DisplayName, *Id); }
};

inline FString LexToString(const FVoiceChatDeviceInfo& DeviceInfo)
{
	return DeviceInfo.ToDebugString();
}

DECLARE_DELEGATE_OneParam(FOnVoiceChatInitializeCompleteDelegate, const FVoiceChatResult& /* Result */);
DECLARE_DELEGATE_OneParam(FOnVoiceChatUninitializeCompleteDelegate, const FVoiceChatResult& /* Result */);
DECLARE_DELEGATE_OneParam(FOnVoiceChatConnectCompleteDelegate, const FVoiceChatResult& /* Result */);
DECLARE_DELEGATE_OneParam(FOnVoiceChatDisconnectCompleteDelegate, const FVoiceChatResult& /* Result */);
DECLARE_DELEGATE_TwoParams(FOnVoiceChatLoginCompleteDelegate, const FString& /* PlayerName */, const FVoiceChatResult& /* Result */);
DECLARE_DELEGATE_TwoParams(FOnVoiceChatLogoutCompleteDelegate, const FString& /* PlayerName */, const FVoiceChatResult& /* Result */);
DECLARE_DELEGATE_TwoParams(FOnVoiceChatChannelJoinCompleteDelegate, const FString& /* ChannelName */, const FVoiceChatResult& /* Result */);
DECLARE_DELEGATE_TwoParams(FOnVoiceChatChannelLeaveCompleteDelegate, const FString& /* ChannelName */, const FVoiceChatResult& /* Result */);

DECLARE_MULTICAST_DELEGATE(FOnVoiceChatAvailableAudioDevicesChangedDelegate);
DECLARE_MULTICAST_DELEGATE(FOnVoiceChatReconnectedDelegate);
DECLARE_MULTICAST_DELEGATE(FOnVoiceChatConnectedDelegate);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnVoiceChatDisconnectedDelegate, const FVoiceChatResult& /* Reason */);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnVoiceChatLoggedInDelegate, const FString& /* PlayerName */);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnVoiceChatLoggedOutDelegate, const FString& /* PlayerName */);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnVoiceChatCallStatsUpdatedDelegate, const FVoiceChatCallStats& /* CallStats */);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnVoiceChatCallStatsUpdatedDelegate2, const FString& /* ChannelName */, const FVoiceChatCallStats& /* CallStats */);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnVoiceChatChannelJoinedDelegate, const FString& /* ChannelName */);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnVoiceChatChannelExitedDelegate, const FString& /* ChannelName */, const FVoiceChatResult& /* Reason */);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnVoiceChatPlayerAddedDelegate, const FString& /* ChannelName */, const FString& /* PlayerName */);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnVoiceChatPlayerRemovedDelegate, const FString& /* ChannelName */, const FString& /* PlayerName */);
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnVoiceChatPlayerTalkingUpdatedDelegate, const FString& /* ChannelName */, const FString& /* PlayerName */, bool /* bIsTalking */);
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnVoiceChatPlayerMuteUpdatedDelegate, const FString& /* ChannelName */, const FString& /* PlayerName */, bool /* bIsMuted */);
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnVoiceChatPlayerVolumeUpdatedDelegate, const FString& /* ChannelName */, const FString& /* PlayerName */, float /* Volume */);

DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnVoiceChatRecordSamplesAvailableDelegate, TArrayView<const int16> /* PcmSamples */, int /* SampleRate */, int /* Channels */);
DECLARE_MULTICAST_DELEGATE_FourParams(FOnVoiceChatAfterCaptureAudioReadDelegate2, const FString& /* ChannelName */, TArrayView<int16> /* PcmSamples */, int /* SampleRate */, int /* Channels */);
DECLARE_MULTICAST_DELEGATE_FiveParams(FOnVoiceChatBeforeCaptureAudioSentDelegate2, const FString& /* ChannelName */, TArrayView<const int16> /* PcmSamples */, int /* SampleRate */, int /* Channels */, bool /* bIsSpeaking */);
DECLARE_MULTICAST_DELEGATE_SixParams(FOnVoiceChatBeforeRecvAudioRenderedDelegate, TArrayView<int16> /* PcmSamples */, int /* SampleRate */, int /* Channels */, bool /* bIsSilence */, const FString& /*ChannelName*/, const FString& /* PlayerName */);

DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnVoiceChatAfterCaptureAudioReadDelegate, TArrayView<int16> /* PcmSamples */, int /* SampleRate */, int /* Channels */);
DECLARE_MULTICAST_DELEGATE_FourParams(FOnVoiceChatBeforeCaptureAudioSentDelegate, TArrayView<const int16> /* PcmSamples */, int /* SampleRate */, int /* Channels */, bool /* bIsSpeaking */);

class IVoiceChatUser
{
public:
	virtual ~IVoiceChatUser() = default;

	/**
	 * Set an implementation specific setting
	 *
	 * @param Name Name of the config setting to set
	 * @param Value Value for the setting
	 */
	virtual void SetSetting(const FString& Name, const FString& Value) = 0;

	/**
	 * Get an implementation specific setting
	 *
	 * @param Name Name of the config setting to get
	 *
	 * @return Current value for the setting
	 */
	virtual FString GetSetting(const FString& Name) = 0;

	/**
	 * Set record volume
	 *
	 * @param Volume Value between 0.0 and 2.0. 1.0 leaves the source volume unchanged.
	 */
	virtual void SetAudioInputVolume(float Volume) = 0;

	/**
	 * Set playback volume
	 *
	 * @param Volume Value between 0.0 and 2.0. 1.0 leaves the source volume unchanged.
	 */
	virtual void SetAudioOutputVolume(float Volume) = 0;

	/**
	 * Get volume of audio input device
	 *
	 * @return Audio input device volume between 0.0 and 2.0. 1.0 leaves the source volume unchanged.
	 */
	virtual float GetAudioInputVolume() const = 0;

	/**
	 * Get volume of audio output device
	 *
	 * @return Audio output device volume between 0.0 and 2.0. 1.0 leaves the source volume unchanged.
	 */
	virtual float GetAudioOutputVolume() const = 0;

	/**
	 * Mute or unmute the audio input device
	 *
	 * @param bIsMuted set to true to mute the device or false to unmute
	 */
	virtual void SetAudioInputDeviceMuted(bool bIsMuted) = 0;

	/**
	 * Mute or unmute the audio output device
	 *
	 * @param bIsMuted set to true to mute the device or false to unmute
	 */
	virtual void SetAudioOutputDeviceMuted(bool bIsMuted) = 0;

	/**
	 * Is the input device muted?
	 *
	 * @return true if the input device is muted
	 */
	virtual bool GetAudioInputDeviceMuted() const = 0;

	/**
	 * Is the output device muted?
	 *
	 * @return true if the input device is muted
	 */
	virtual bool GetAudioOutputDeviceMuted() const = 0;

	/**
	 * Get a list of available audio input device names
	 *
	 * @return Array of audio input devices names
	 */
	UE_DEPRECATED(4.26, "GetAvailableInputDevices is deprecated. Please use GetAvailableInputDeviceInfos instead!")
	TArray<FString> GetAvailableInputDevices() const
	{
		TArray<FString> DeviceNames;
		for (const FVoiceChatDeviceInfo& DeviceInfo : GetAvailableInputDeviceInfos())
		{
			DeviceNames.Emplace(DeviceInfo.DisplayName);
		}
		return DeviceNames;
	}

	/**
	 * Get a list of available audio input devices
	 *
	 * @return Array of audio input devices
	 */
	virtual TArray<FVoiceChatDeviceInfo> GetAvailableInputDeviceInfos() const = 0;

	/**
	 * Get a list of available audio output device names
	 *
	 * @return Array of audio output device names
	 */
	UE_DEPRECATED(4.26, "GetAvailableOutputDevices is deprecated. Please use GetAvailableOutputDeviceInfos instead!")
	TArray<FString> GetAvailableOutputDevices() const
	{
		TArray<FString> DeviceNames;
		for (const FVoiceChatDeviceInfo& DeviceInfo : GetAvailableOutputDeviceInfos())
		{
			DeviceNames.Emplace(DeviceInfo.DisplayName);
		}
		return DeviceNames;
	}

	/**
	 * Get a list of available audio output devices
	 *
	 * @return Array of audio output devices
	 */
	virtual TArray<FVoiceChatDeviceInfo> GetAvailableOutputDeviceInfos() const = 0;

	/**
	 * Delegate broadcast whenever the available audio devices change. Any cached values from GetAvailableInputDevices or GetAvailableOutputDevices should be discarded and requeried
	 */
	virtual FOnVoiceChatAvailableAudioDevicesChangedDelegate& OnVoiceChatAvailableAudioDevicesChanged() = 0;

	/**
	 * Set the audio input device to use
	 *
	 * @param InputDevice String from GetAvailableInputDevice. If empty, will use the default input device
	 */
	UE_DEPRECATED(4.26, "SetInputDevice is deprecated. Please use SetInputDeviceId instead!")
	void SetInputDevice(const FString& InputDeviceName)
	{
		FString InputDeviceId = InputDeviceName;
		for (const FVoiceChatDeviceInfo& DeviceInfo : GetAvailableInputDeviceInfos())
		{
			if (InputDeviceName == DeviceInfo.DisplayName)
			{
				InputDeviceId = DeviceInfo.Id;
				break;
			}
		}
		SetInputDeviceId(InputDeviceId);
	}

	/**
	 * Set the audio input device to use
	 *
	 * @param InputDeviceId Device Id from GetAvailableInputDeviceInfos. If empty, will use the default input device
	 */
	virtual void SetInputDeviceId(const FString& InputDeviceId) = 0;

	/**
	 * Set the audio output device to use
	 *
	 * @param OutputDevice String from GetAvailableOutputDevice. If empty, will use the default output device
	 */
	UE_DEPRECATED(4.26, "SetOutputDevice is deprecated. Please use SetOutputDeviceId instead!")
	void SetOutputDevice(const FString& OutputDeviceName)
	{
		FString OutputDeviceId = OutputDeviceName;
		for (const FVoiceChatDeviceInfo& DeviceInfo : GetAvailableOutputDeviceInfos())
		{
			if (OutputDeviceName == DeviceInfo.DisplayName)
			{
				OutputDeviceId = DeviceInfo.Id;
				break;
			}
		}
		SetOutputDeviceId(OutputDeviceId);
	}

	/**
	 * Set the audio output device to use
	 *
	 * @param OutputDeviceId Device Id from GetAvailableOutputDeviceInfos. If empty, will use the default output device
	 */
	virtual void SetOutputDeviceId(const FString& OutputDeviceId) = 0;

	/**
	 * Get the display name of the audio input device that is being used
	 */
	UE_DEPRECATED(4.26, "GetInputDevice is deprecated. Please use GetInputDeviceInfo instead!")
	FString GetInputDevice() const
	{
		return GetInputDeviceInfo().DisplayName;
	}

	/**
	 * Get the device info of the audio input device that is being used
	 */
	virtual FVoiceChatDeviceInfo GetInputDeviceInfo() const = 0;

	/**
	 * Get the display name of the audio output device that is being used
	 */
	UE_DEPRECATED(4.26, "GetOutputDevice is deprecated. Please use GetOutputDeviceInfo instead!")
	FString GetOutputDevice() const
	{
		return GetOutputDeviceInfo().DisplayName;
	}

	/**
	 * Get the device info of the audio output device that is being used
	 */
	virtual FVoiceChatDeviceInfo GetOutputDeviceInfo() const = 0;

	/**
	 * Get the display name of the default audio input device
	 */
	UE_DEPRECATED(4.26, "GetDefaultInputDevice is deprecated. Please use GetDefaultInputDeviceInfo instead!")
	FString GetDefaultInputDevice() const
	{
		return GetDefaultInputDeviceInfo().DisplayName;
	};

	/**
	 * Get the device info of the default audio input device
	 */
	virtual FVoiceChatDeviceInfo GetDefaultInputDeviceInfo() const = 0;

	/**
	 * Get the display name of the default audio output device
	 */
	UE_DEPRECATED(4.26, "GetDefaultOutputDevice is deprecated. Please use GetDefaultOutputDeviceInfo instead!")
	FString GetDefaultOutputDevice() const
	{
		return GetDefaultOutputDeviceInfo().DisplayName;
	}

	/**
	 * Get the device info of the default audio output device
	 */
	virtual FVoiceChatDeviceInfo GetDefaultOutputDeviceInfo() const = 0;

	/**
	 * Login to the connected voice server
	 *
	 * @param PlatformId Platform user Id for the player logging in
	 * @param PlayerName Name of the account to login to
	 * @param Credentials Token or password
	 * @param Delegate delegate called once login completes
	 */
	virtual void Login(FPlatformUserId PlatformId, const FString& PlayerName, const FString& Credentials, const FOnVoiceChatLoginCompleteDelegate& Delegate) = 0;

	/**
	 * Logout from the connected voice server
	 *
	 * @param Delegate delegate called once logout completes
	 */
	virtual void Logout(const FOnVoiceChatLogoutCompleteDelegate& Delegate) = 0;

	/**
	 * Are we logging in?
	 *
	 * @return true if we are logging in to the voice server
	 */
	virtual bool IsLoggingIn() const = 0;

	/**
	 * Are we logged in?
	 *
	 * @return true if we are logged in to the voice server
	 */
	virtual bool IsLoggedIn() const = 0;

	/**
	 * Delegate triggered when we are logged in to voice chat
	 */
	virtual FOnVoiceChatLoggedInDelegate& OnVoiceChatLoggedIn() = 0;

	/**
	 * Delegate triggered when we are logged out from voice chat
	 */
	virtual FOnVoiceChatLoggedOutDelegate& OnVoiceChatLoggedOut() = 0;

	/**
	 * Get the player name used to log in
	 *
	 * @return player name used to log in
	 */
	virtual FString GetLoggedInPlayerName() const = 0;

	/**
	 * Block communication with players
	 *
	 * Adds the PlayerNames to the block list, muting them in all current and subsequently joined channels.
	 * A block for a player persists until either:
	 *		- The local user logs out (e.g. by calling Logout/Disconnect/Uninitialize, or receiving equivalent events)
	 *		- A subsequent call to UnblockPlayers is made with that PlayerName.
	 *
	 * @param PlayerNames List of players to block communication with
	 */
	virtual void BlockPlayers(const TArray<FString>& PlayerNames) = 0;

	/**
	 * Unblock communication with players
	 *
	 * @param PlayerNames List of players to unblock communication with
	 */
	virtual void UnblockPlayers(const TArray<FString>& PlayerNames) = 0;

	/**
	 * Join a voice channel
	 *
	 * @param ChannelName Channel to join
	 * @param ChannelCredentials password or token
	 * @param ChannelType NonPositional, Positional, or Echo
	 * @param Delegate delegate called once join completes
	 * @param Channel3dProperties Optional parameter to set 3d/Positional channel properties
	 */
	virtual void JoinChannel(const FString& ChannelName, const FString& ChannelCredentials, EVoiceChatChannelType ChannelType, const FOnVoiceChatChannelJoinCompleteDelegate& Delegate, TOptional<FVoiceChatChannel3dProperties> Channel3dProperties = TOptional<FVoiceChatChannel3dProperties>()) = 0;

	/**
	 * Leave a voice channel
	 * 
	 * @param ChannelName Channel to leave
	 * @param Delegate delegate called once leave completes
	 */
	virtual void LeaveChannel(const FString& ChannelName, const FOnVoiceChatChannelLeaveCompleteDelegate& Delegate) = 0;

	/**
	 * Delegate triggered when we join a voice channel
	 */
	virtual FOnVoiceChatChannelJoinedDelegate& OnVoiceChatChannelJoined() = 0;

	/**
	 * Delegate triggered when we leave a voice channel
	 */
	virtual FOnVoiceChatChannelExitedDelegate& OnVoiceChatChannelExited() = 0;

	/**
	 * Delegate triggered when a call is ended, providing the stats for the call.
	 */
	UE_DEPRECATED(5.8, "Please use OnVoiceChatCallStatsUpdated2 which adds the ChannelName param")
	virtual FOnVoiceChatCallStatsUpdatedDelegate& OnVoiceChatCallStatsUpdated() = 0;

	/**
	 * Delegate triggered when a call ends on a specific channel, providing the cumulative stats for
	 * the call. Replaces OnVoiceChatCallStatsUpdated to allow per-channel routing in implementations
	 * that may have multiple concurrent voice rooms.
	 */
	virtual FOnVoiceChatCallStatsUpdatedDelegate2& OnVoiceChatCallStatsUpdated2() = 0;

	/**
	 * Returns the current cumulative call-quality stats for the named channel, if available, or an
	 * empty optional if the channel is not currently joined or if the implementation does not produce
	 * stats. The values are running totals since the channel was joined.
	 */
	virtual TOptional<FVoiceChatCallStats> GetChannelCallStats(const FString& ChannelName) const = 0;

	/**
	 * Set the 3d position of the player
	 *
	 * @param ChannelName
	 * @param Position
	 */
	virtual void Set3DPosition(const FString& ChannelName, const FVector& Position) = 0;

	/**
	 * Get an array of channels the user is in
	 *
	 * @return Array of connected channel names
	 */
	virtual TArray<FString> GetChannels() const = 0;

	/**
	 * Get an array of players in a channel
	 *
	 * @param ChannelName Channel to get the list of players for
	 * @return Array of names of players in Channel
	 */
	virtual TArray<FString> GetPlayersInChannel(const FString& ChannelName) const = 0;

	/**
	 * Get an array of players in a channel
	 *
	 * @param ChannelName Channel to get the type of
	 * @return Channel type
	 */
	virtual EVoiceChatChannelType GetChannelType(const FString& ChannelName) const = 0;

	/**
	 * Delegate triggered when a player is added to a channel
	 */
	virtual FOnVoiceChatPlayerAddedDelegate& OnVoiceChatPlayerAdded() = 0;

	/**
	 * Delegate triggered when a player is removed from a channel
	 */
	virtual FOnVoiceChatPlayerRemovedDelegate& OnVoiceChatPlayerRemoved() = 0;

	/**
	 * @param PlayerName Player to get the talking state of
	 * @return true if player is talking
	 */
	virtual bool IsPlayerTalking(const FString& PlayerName) const = 0;

	/**
	 * Delegate triggered when a player's talking state is updated
	 */
	virtual FOnVoiceChatPlayerTalkingUpdatedDelegate& OnVoiceChatPlayerTalkingUpdated() = 0;

	/**
	 * Mute or unmute a player
	 *
	 * @param PlayerName Player to mute
	 * @param bMuted true if the player should be muted
	 */
	virtual void SetPlayerMuted(const FString& PlayerName, bool bMuted) = 0;

	/**
	 * @param PlayerName Player to get the mute state of
	 * @return true if player is muted
	 */
	virtual bool IsPlayerMuted(const FString& PlayerName) const = 0;

	/**
	 * Mute or unmute a player in a given channel.
	 * Only has effect if the player is currently in the channel, and only until you/they leave the channel.
	 * i.e. this is expected to be [re]called by OnVoiceChatPlayerAdded delegates.
	 *
	 * @param ChannelName Channel in which to mute the player
	 * @param PlayerName Player to mute in channel
	 * @param bMuted true if the player should be muted in the specified channel
	 */
	virtual void SetChannelPlayerMuted(const FString& ChannelName, const FString& PlayerName, bool bAudioMuted) = 0;

	/**
	 * @param ChannelName Channel in which the mute state will be checked
	 * @param PlayerName Player to get the mute state of in channel
	 * @return true if player is muted
	 */
	virtual bool IsChannelPlayerMuted(const FString& ChannelName, const FString& PlayerName) const = 0;

	/**
	 * Delegate triggered when a player's muted state is updated, usually as a result of calling SetPlayerMuted
	 */
	virtual FOnVoiceChatPlayerMuteUpdatedDelegate& OnVoiceChatPlayerMuteUpdated() = 0;

	/**
	 * Set player volume
	 *
	 * @param PlayerName Player to adjust the volume for
	 * @param Volume Value between 0.0 and 2.0. 1.0 leaves the source volume unchanged.
	 */
	virtual void SetPlayerVolume(const FString& PlayerName, float Volume) = 0;

	/**
	 * @param PlayerName Player to get the volume of
	 * @return Volume Value between 0.0 and 2.0. 1.0 leaves the source volume unchanged.
	 */
	virtual float GetPlayerVolume(const FString& PlayerName) const = 0;

	/**
	 * Delegate triggered when a player's volume is updated, usually as a result of calling SetPlayerVolume
	 */
	virtual FOnVoiceChatPlayerVolumeUpdatedDelegate& OnVoiceChatPlayerVolumeUpdated() = 0;

	/**
	 * Transmit to all channels that you are currently in
	 */
	virtual void TransmitToAllChannels() = 0;
	
	/**
	 * Stop transmitting to all channels that you are currently in. This does not apply to channels you will join.
	 */
	virtual void TransmitToNoChannels() = 0;

	/**
	 * Transmit to Specified set of channels
	 * @param ChannelNames Channels to transmit to
	 */
	virtual void TransmitToSpecificChannels(const TSet<FString>& ChannelNames) = 0;

	/**
	 * Add specific channels to transmit
	 * @param ChannelNames New channels to add to the current transmit set
	 */
	virtual void AddSpecificChannelsToTransmit(const TSet<FString>& ChannelNames) { TransmitToSpecificChannels(GetTransmitChannels().Union(ChannelNames)); }

	/**
	 * Remove specific channels from transmit
	 * @param ChannelNames Channels to remove from the current transmit set
	 */
	virtual void RemoveSpecificChannelsFromTransmit(const TSet<FString>& ChannelNames) { TransmitToSpecificChannels(GetTransmitChannels().Difference(ChannelNames)); }

	/**
	 * Get the current transmit mode
	 *
	 * @return Transmit mode. If it is EVoiceChatTransmitMode::SpecificChannels, the current channels can be retrieved using GetTransmitChannels()
	 */
	virtual EVoiceChatTransmitMode GetTransmitMode() const = 0;

	/**
	 * Get all channels we are transmitting to
	 *
	 * @return Channels	 we are transmitting to. Will be empty if GetTransmissionMode() returned a value other than EVoiceChatTransmitMode::SpecificChannels
	 */
	virtual TSet<FString> GetTransmitChannels() const = 0;

	/*
	 * Start recording
	 *
	 * @param Delegate This will be called from audio thread.
	 *
	 * @return Delegate handle to pass to stop recording
	 */
	virtual FDelegateHandle StartRecording(const FOnVoiceChatRecordSamplesAvailableDelegate::FDelegate& Delegate) = 0;

	/*
	 * Stop recording
	 *
	 * @param Handle Delegate handle returned from a StartRecording call
	 */
	virtual void StopRecording(FDelegateHandle Handle) = 0;

	/**
	 * Delegates called from audio thread.
	 */
	virtual FDelegateHandle RegisterOnVoiceChatAfterCaptureAudioReadDelegate(const FOnVoiceChatAfterCaptureAudioReadDelegate2::FDelegate& Delegate) = 0;
	virtual void UnregisterOnVoiceChatAfterCaptureAudioReadDelegate(FDelegateHandle Handle) = 0;

	virtual FDelegateHandle RegisterOnVoiceChatBeforeCaptureAudioSentDelegate(const FOnVoiceChatBeforeCaptureAudioSentDelegate2::FDelegate& Delegate) = 0;
	virtual void UnregisterOnVoiceChatBeforeCaptureAudioSentDelegate(FDelegateHandle Handle) = 0;
	
	virtual FDelegateHandle RegisterOnVoiceChatBeforeRecvMixedAudioRenderedDelegate(const FOnVoiceChatBeforeRecvAudioRenderedDelegate::FDelegate& Delegate) = 0;
	virtual void UnregisterOnVoiceChatBeforeRecvMixedAudioRenderedDelegate(FDelegateHandle Handle) = 0;
	
	virtual FDelegateHandle RegisterOnVoiceChatBeforeRecvUnmixedAudioRenderedDelegate(const FOnVoiceChatBeforeRecvAudioRenderedDelegate::FDelegate& Delegate) = 0;
	virtual void UnregisterOnVoiceChatBeforeRecvUnmixedAudioRenderedDelegate(FDelegateHandle Handle) = 0;

	UE_DEPRECATED(5.7, "Please use RegisterOnVoiceChatBeforeRecvMixedAudioRenderedDelegate instead. Also now possible to use RegisterOnVoiceChatBeforeRecvUnmixedAudioRenderedDelegate for receiving unmixed audio")
	virtual FDelegateHandle RegisterOnVoiceChatBeforeRecvAudioRenderedDelegate(const FOnVoiceChatBeforeRecvAudioRenderedDelegate::FDelegate& Delegate) final
	{
		return RegisterOnVoiceChatBeforeRecvMixedAudioRenderedDelegate(Delegate);
	}
	UE_DEPRECATED(5.7, "Please use UnregisterOnVoiceChatBeforeRecvMixedAudioRenderedDelegate instead. Also now possible to use UnregisterOnVoiceChatBeforeRecvUnmixedAudioRenderedDelegate for unmixed audio")
	virtual void UnregisterOnVoiceChatBeforeRecvAudioRenderedDelegate(FDelegateHandle Handle) final
	{
		UnregisterOnVoiceChatBeforeRecvMixedAudioRenderedDelegate(Handle);
	}

	UE_DEPRECATED(5.6, "Please pass a FOnVoiceChatBeforeCaptureAudioSentDelegate2, which adds the ChannelName param")
	virtual FDelegateHandle RegisterOnVoiceChatAfterCaptureAudioReadDelegate(const FOnVoiceChatAfterCaptureAudioReadDelegate::FDelegate& Delegate) final
	{
		return RegisterOnVoiceChatAfterCaptureAudioReadDelegate(FOnVoiceChatAfterCaptureAudioReadDelegate2::FDelegate::CreateLambda([Delegate](const FString& /* ChannelName */, TArrayView<int16> PcmSamples, int SampleRate, int Channels)
			{
				Delegate.ExecuteIfBound(PcmSamples, SampleRate, Channels);
			}));
	}

	UE_DEPRECATED(5.6, "Please pass a FOnVoiceChatBeforeCaptureAudioSentDelegate2, which adds the ChannelName param")
		virtual FDelegateHandle RegisterOnVoiceChatBeforeCaptureAudioSentDelegate(const FOnVoiceChatBeforeCaptureAudioSentDelegate::FDelegate& Delegate) final
	{
		return RegisterOnVoiceChatBeforeCaptureAudioSentDelegate(FOnVoiceChatBeforeCaptureAudioSentDelegate2::FDelegate::CreateLambda([Delegate](const FString& /* ChannelName */, TArrayView<const int16> PcmSamples, int SampleRate, int Channels, bool bIsSpeaking)
			{
				Delegate.ExecuteIfBound(PcmSamples, SampleRate, Channels, bIsSpeaking);
			}));
	}

	/**
	 * Generate Login/Join tokens. Should be used for development purposes only
	 */
	virtual FString InsecureGetLoginToken(const FString& PlayerName) = 0;
	virtual FString InsecureGetJoinToken(const FString& ChannelName, EVoiceChatChannelType ChannelType, TOptional<FVoiceChatChannel3dProperties> Channel3dProperties = TOptional<FVoiceChatChannel3dProperties>()) = 0;

};

class IVoiceChat : public IVoiceChatUser, public IModularFeature
{
public:
	/**
	 * Get the IVoiceChat modular feature instance
	 */
	static IVoiceChat* Get()
	{
		if (IModularFeatures::Get().IsModularFeatureAvailable(GetModularFeatureName()))
		{
			return &IModularFeatures::Get().GetModularFeature<IVoiceChat>(GetModularFeatureName());
		}
		return nullptr;
	}

	static FName GetModularFeatureName()
	{
		static FName FeatureName = FName(TEXT("VoiceChat"));
		return FeatureName;
	}

	/**
	 * Initialize VoiceChat synchronously
	 */
	virtual bool Initialize() = 0;

	/**
	 * Initialize VoiceChat asynchronously
	 */
	virtual void Initialize(const FOnVoiceChatInitializeCompleteDelegate& Delegate) = 0;

	/**
	 * Uninitialize VoiceChat synchronously
	 */
	virtual bool Uninitialize() = 0;

	/**
	 * Uninitialize VoiceChat asynchronously
	 */
	virtual void Uninitialize(const FOnVoiceChatUninitializeCompleteDelegate& Delegate) = 0;

	/**
	 * Is voice chat initialized?
	 *
	 * @return true if voice chat is initialized
	 */
	virtual bool IsInitialized() const = 0;

	/**
	 * Connect to a voice server
	 *
	 * @param Delegate delegate called once connect completes
	 */
	virtual void Connect(const FOnVoiceChatConnectCompleteDelegate& Delegate) = 0;

	/**
	 * Disconnect from a voice server
	 *
	 * @param Delegate delegate called once disconnect completes
	 */
	virtual void Disconnect(const FOnVoiceChatDisconnectCompleteDelegate& Delegate) = 0;

	/**
	* Are we connecting to the voice server?
	*
	* @return true if we are connecting to the voice server
	*/
	virtual bool IsConnecting() const = 0;

	/**
	 * Are we connected to the voice server?
	 *
	 * @return true if we are connected to the voice server
	 */
	virtual bool IsConnected() const = 0;

	/**
	 * Delegate triggered when we are connected to voice chat
	 */
	virtual FOnVoiceChatConnectedDelegate& OnVoiceChatConnected() = 0;

	/**
	 * Delegate triggered when we are disconnected from voice chat
	 */
	virtual FOnVoiceChatDisconnectedDelegate& OnVoiceChatDisconnected() = 0;

	/**
	 * Delegate triggered when we are reconnected to voice chat in cases where the underlying implementation disconnected
	 * and then reconnected. This is most commonly seen when an application resumes after being suspended. If the disconnect
	 * is unexpected, OnVoiceChatDisconnected will be called instead.
	 */
	virtual FOnVoiceChatReconnectedDelegate& OnVoiceChatReconnected() = 0;

	/**
	 * Allocate an interface for an additional user. The interface must be released with ReleaseUser.
	 *
	 * @return new instance of IVoiceChatUser, or nullptr if implementation does not support multiple users
	 */
	virtual IVoiceChatUser* CreateUser() = 0;

	/**
	 * Release an interface for an additional user.
	 * 
	 * @param VoiceChatUser the user interface to release.
	 */
	virtual void ReleaseUser(IVoiceChatUser* VoiceChatUser) = 0;
};
