// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "VoiceChatResult.h"

#define VOICECHAT_ERROR(...) FVoiceChatResult::CreateError(TEXT("errors.com.epicgames.voicechat"), __VA_ARGS__)

namespace VoiceChat
{
	namespace Errors
	{
		inline FVoiceChatResult NotEnabled() { return VOICECHAT_ERROR(EVoiceChatResult::InvalidState, TEXT("not_enabled")); }
		inline FVoiceChatResult NotInitialized() { return VOICECHAT_ERROR(EVoiceChatResult::NotInitialized, TEXT("not_initialized")); }
		inline FVoiceChatResult NotConnected() { return VOICECHAT_ERROR(EVoiceChatResult::NotConnected, TEXT("not_connected")); }
		inline FVoiceChatResult NotLoggedIn() { return VOICECHAT_ERROR(EVoiceChatResult::NotLoggedIn, TEXT("not_logged_in")); }

		inline FVoiceChatResult NotPermitted() { return VOICECHAT_ERROR(EVoiceChatResult::NotPermitted, TEXT("not_permitted")); }
		inline FVoiceChatResult Throttled()	{ return VOICECHAT_ERROR(EVoiceChatResult::Throttled, TEXT("throttled")); }

		inline FVoiceChatResult InvalidState() { return VOICECHAT_ERROR(EVoiceChatResult::InvalidState, TEXT("invalid_state")); }
		inline FVoiceChatResult MissingConfig() { return VOICECHAT_ERROR(EVoiceChatResult::InvalidState, TEXT("missing_config")); }
		inline FVoiceChatResult AlreadyInProgress() { return VOICECHAT_ERROR(EVoiceChatResult::InvalidState, TEXT("already_in_progress")); }
		inline FVoiceChatResult DisconnectInProgress() { return VOICECHAT_ERROR(EVoiceChatResult::InvalidState, TEXT("disconnect_in_progress")); }
		inline FVoiceChatResult OtherUserLoggedIn() { return VOICECHAT_ERROR(EVoiceChatResult::InvalidState, TEXT("other_user_logged_in")); }
		inline FVoiceChatResult NotInChannel() { return VOICECHAT_ERROR(EVoiceChatResult::InvalidState, TEXT("not_in_channel")); }
		inline FVoiceChatResult ChannelJoinInProgress() { return VOICECHAT_ERROR(EVoiceChatResult::InvalidState, TEXT("channel_join_in_progress")); }
		inline FVoiceChatResult ChannelLeaveInProgress() { return VOICECHAT_ERROR(EVoiceChatResult::InvalidState, TEXT("channel_leave_in_progress")); }
		inline FVoiceChatResult PlatformPartyChatActive() { return VOICECHAT_ERROR(EVoiceChatResult::InvalidState, TEXT("platform_party_chat_active")); }

		inline FVoiceChatResult InvalidArgument(const FString& ErrorDesc = FString()) { return VOICECHAT_ERROR(EVoiceChatResult::InvalidArgument, TEXT("invalid_argument"), ErrorDesc); }
		inline FVoiceChatResult CredentialsInvalid(const FString& ErrorDesc = FString()) { return VOICECHAT_ERROR(EVoiceChatResult::CredentialsInvalid, TEXT("credentials_invalid"), ErrorDesc); }
		inline FVoiceChatResult CredentialsExpired() { return VOICECHAT_ERROR(EVoiceChatResult::CredentialsExpired, TEXT("credentials_expired")); }

		inline FVoiceChatResult ClientTimeout() { return VOICECHAT_ERROR(EVoiceChatResult::ClientTimeout, TEXT("client_timeout")); }
		inline FVoiceChatResult ServerTimeout() { return VOICECHAT_ERROR(EVoiceChatResult::ServerTimeout, TEXT("server_timeout")); }
		inline FVoiceChatResult DnsFailure() { return VOICECHAT_ERROR(EVoiceChatResult::DnsFailure, TEXT("dns_failure")); }
		inline FVoiceChatResult ConnectionFailure() { return VOICECHAT_ERROR(EVoiceChatResult::ConnectionFailure, TEXT("connection_failure")); }
	}
}

#undef VOICECHAT_ERROR