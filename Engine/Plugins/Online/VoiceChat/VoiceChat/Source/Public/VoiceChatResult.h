// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"

enum class EVoiceChatResult
{
	// The operation succeeded
	Success = 0,

	// Common state errors
	InvalidState,
	NotInitialized,
	NotConnected,
	NotLoggedIn,
	NotPermitted,
	Throttled,

	// Common argument errors
	InvalidArgument,
	CredentialsInvalid,
	CredentialsExpired,

	// Common connection errors
	ClientTimeout,
	ServerTimeout,
	DnsFailure,
	ConnectionFailure,

	// Error does not map to any common categories
	ImplementationError
};

struct FVoiceChatResult
{
	/** Success, or an error category */
	EVoiceChatResult ResultCode;
	/** If we failed, the code for the error */
	FString ErrorCode;
	/** If we failed, a numeric error from the implementation */
	int ErrorNum = 0;
	/** If we failed, more details about the error condition */
	FString ErrorDesc;

	bool IsSuccess() const { return ResultCode == EVoiceChatResult::Success; }
	
	bool operator==(const FVoiceChatResult& Other) const
	{
		return ResultCode == Other.ResultCode && ErrorCode == Other.ErrorCode;
	}

	bool operator!=(const FVoiceChatResult& Other) const
	{
		return !(FVoiceChatResult::operator==(Other));
	}

	FVoiceChatResult(EVoiceChatResult ResultCode) : ResultCode(ResultCode) {}
	FVoiceChatResult(EVoiceChatResult ResultCode, const FString& ErrorCode) : ResultCode(ResultCode), ErrorCode(ErrorCode) {}
	FVoiceChatResult(EVoiceChatResult ResultCode, const FString& ErrorCode, const FString& ErrorDesc) : ResultCode(ResultCode), ErrorCode(ErrorCode), ErrorDesc(ErrorDesc) {}
	FVoiceChatResult(EVoiceChatResult ResultCode, const FString& ErrorCode, int ErrorNum) : ResultCode(ResultCode), ErrorCode(ErrorCode), ErrorNum(ErrorNum) {}
	FVoiceChatResult(EVoiceChatResult ResultCode, const FString& ErrorCode, int ErrorNum, const FString& ErrorDesc) : ResultCode(ResultCode), ErrorCode(ErrorCode), ErrorDesc(ErrorDesc) {}

	static FVoiceChatResult CreateSuccess() { return FVoiceChatResult(EVoiceChatResult::Success); }

	/** Create factory for proper namespacing */
	static FVoiceChatResult CreateError(const FString& ErrorNamespace, EVoiceChatResult ResultCode, const FString& ErrorCode, const FString& ErrorDesc = FString())
	{
		FVoiceChatResult Error(ResultCode, ErrorCode, ErrorDesc);
		if (!Error.ErrorCode.IsEmpty() && !Error.ErrorCode.StartsWith(TEXT("errors.com.")))
		{
			FString NamespacedError = ErrorNamespace;
			NamespacedError += TEXT(".");
			NamespacedError += MoveTemp(Error.ErrorCode);
			Error.ErrorCode = MoveTemp(NamespacedError);
		}

		return Error;
	}

	/** Create factory for proper namespacing */
	static FVoiceChatResult CreateError(const FString& ErrorNamespace, EVoiceChatResult ResultCode, const FString& ErrorCode, int ErrorNum, const FString& ErrorDesc = FString())
	{
		FVoiceChatResult Error = CreateError(ErrorNamespace, ResultCode, ErrorCode, ErrorDesc);
		Error.ErrorNum = ErrorNum;
		return Error;
	}
};

inline FString LexToString(const EVoiceChatResult Result)
{
	switch (Result)
	{
	case EVoiceChatResult::Success: return TEXT("Success");
	// Common State errors
	case EVoiceChatResult::InvalidState: return TEXT("InvalidState");
	case EVoiceChatResult::NotInitialized: return TEXT("NotInitialized");
	case EVoiceChatResult::NotConnected: return TEXT("NotConnected");
	case EVoiceChatResult::NotLoggedIn: return TEXT("NotLoggedIn");
	case EVoiceChatResult::NotPermitted: return TEXT("NotPermitted");
	case EVoiceChatResult::Throttled: return TEXT("Throttled");
	// Common argument errors
	case EVoiceChatResult::InvalidArgument: return TEXT("InvalidArgument");
	case EVoiceChatResult::CredentialsInvalid: return TEXT("AccessTokenInvalid");
	case EVoiceChatResult::CredentialsExpired: return TEXT("AccessTokenExpired");
	// Common connection errors 
	case EVoiceChatResult::ClientTimeout: return TEXT("ClientTimeout");
	case EVoiceChatResult::ServerTimeout: return TEXT("ServerTimeout");
	case EVoiceChatResult::DnsFailure: return TEXT("DnsFailure");
	case EVoiceChatResult::ConnectionFailure: return TEXT("ConnectionFailure");
	// Error does not map to any common categories
	case EVoiceChatResult::ImplementationError: return TEXT("ImplementationError");
	default:
		checkNoEntry();
		return TEXT("UNKNOWN");
	}
}

inline FString LexToString(const FVoiceChatResult& Result)
{
	if (Result.IsSuccess())
	{
		return TEXT("Success");
	}
	else if (Result.ErrorDesc.IsEmpty())
	{
		return FString::Printf(TEXT("Failed with ResultCode=[%s], ErrorCode=[%s]"), *LexToString(Result.ResultCode), *Result.ErrorCode);
	}
	else
	{
		return FString::Printf(TEXT("Failed with ResultCode=[%s], ErrorCode=[%s], ErrorDesc=[%s]"), *LexToString(Result.ResultCode), *Result.ErrorCode, *Result.ErrorDesc);
	}
}