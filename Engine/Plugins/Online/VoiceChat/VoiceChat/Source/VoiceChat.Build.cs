// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class VoiceChat : ModuleRules
	{
		public VoiceChat(ReadOnlyTargetRules Target) : base(Target)
		{
			CppCompileWarningSettings.ModuleIncludePathWarningLevel = WarningLevel.Off;

			PublicIncludePaths.Add(Path.Combine(ModuleDirectory, "Public"));

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core"
				}
			);
		}
	}
}
