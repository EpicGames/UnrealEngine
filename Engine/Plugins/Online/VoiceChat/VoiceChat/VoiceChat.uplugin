{
	"FileVersion" : 3,
	
	"FriendlyName" : "Voice Chat Interface",
	"Version" : 1,
	"VersionName" : "1.0",
	"Description" : "Voice Chat Interface",
	"Category" : "Online Platform",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"EnabledByDefault" : false,
	
	"Modules" :
	[
		{
			"Name" : "VoiceChat",
			"Type" : "ClientOnly"
		}
	]
}
